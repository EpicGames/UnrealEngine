// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class AssetDefinition : ModuleRules
{
	public AssetDefinition(ReadOnlyTargetRules Target) : base(Target)
	{		
        PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"AssetRegistry",
				"Engine",
                "InputCore",
				"Slate",
				"SlateCore",
				"EditorSubsystem"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
			}
		);
		
		PrivateIncludePaths.AddRange(
        	new string[] {
        	}
        );
	}
}
