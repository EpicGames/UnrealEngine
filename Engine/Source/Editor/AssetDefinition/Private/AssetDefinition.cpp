// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinition.h"

#include "AssetDefinitionAssetInfo.h"
#include "AssetDefinitionRegistry.h"
#include "IAssetStatusInfoProvider.h"
#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ISourceControlState.h"
#include "Misc/AssetFilterData.h"
#include "EditorFramework/AssetImportData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinition)

#define LOCTEXT_NAMESPACE "UAssetDefinition"

FAssetCategoryPath EAssetCategoryPaths::Basic(LOCTEXT("Basic", "Basic"));
FAssetCategoryPath EAssetCategoryPaths::Animation(LOCTEXT("Animation", "Animation"));
FAssetCategoryPath EAssetCategoryPaths::Material(LOCTEXT("Material", "Material"));
FAssetCategoryPath EAssetCategoryPaths::Audio(LOCTEXT("Audio", "Audio"));
FAssetCategoryPath EAssetCategoryPaths::Physics(LOCTEXT("Physics", "Physics"));
FAssetCategoryPath EAssetCategoryPaths::UI(LOCTEXT("UserInterface", "User Interface"));
FAssetCategoryPath EAssetCategoryPaths::Misc(LOCTEXT("Miscellaneous", "Miscellaneous"));
FAssetCategoryPath EAssetCategoryPaths::Gameplay(LOCTEXT("Gameplay", "Gameplay"));
FAssetCategoryPath EAssetCategoryPaths::AI(LOCTEXT("AI", "Artificial Intelligence"));
FAssetCategoryPath EAssetCategoryPaths::Blueprint(LOCTEXT("Blueprint", "Blueprint"));
FAssetCategoryPath EAssetCategoryPaths::Texture(LOCTEXT("Texture", "Texture"));
FAssetCategoryPath EAssetCategoryPaths::Foliage(LOCTEXT("Foliage", "Foliage"));
FAssetCategoryPath EAssetCategoryPaths::Input(LOCTEXT("Input", "Input"));
FAssetCategoryPath EAssetCategoryPaths::FX(LOCTEXT("FX", "FX"));
FAssetCategoryPath EAssetCategoryPaths::Cinematics(LOCTEXT("Cinematics", "Cinematics"));
FAssetCategoryPath EAssetCategoryPaths::Data(LOCTEXT("Data", "Data"));
FAssetCategoryPath EAssetCategoryPaths::Media(LOCTEXT("Media", "Media"));
FAssetCategoryPath EAssetCategoryPaths::World(LOCTEXT("World", "World"));

namespace UE::AssetDefinition::Status
{
	EVisibility GetDirtyStatusVisibility(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider)
	{
		EVisibility DirtyStatusVisibility = EVisibility::Collapsed;
		if (!InAssetStatusInfoProvider.IsValid())
		{
			return DirtyStatusVisibility;
		}

		if (const UPackage* Package = InAssetStatusInfoProvider->FindPackage())
		{
			DirtyStatusVisibility = Package->IsDirty() ? EVisibility::Visible : EVisibility::Collapsed;
		}
		return DirtyStatusVisibility;
	}

	const FSlateBrush* GetSourceControlStatusBrush(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider)
	{
		const FSlateBrush* SourceControlBrush = FAppStyle::GetNoBrush();
		if (!InAssetStatusInfoProvider.IsValid())
		{
			return SourceControlBrush;
		}

		if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			const FString FileName = InAssetStatusInfoProvider->TryGetFilename();
			if (FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(FileName, EStateCacheUsage::Use))
			{
				FSlateIcon SCCIcon = SourceControlState->GetIcon();
				if (SCCIcon.IsSet())
				{
					SourceControlBrush = SCCIcon.GetIcon();
				}
			}
		}
		return SourceControlBrush;
	}

	const FSlateBrush* GetSourceControlStatusOverlayBrush(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider)
	{
		const FSlateBrush* SourceControlStatusOverlay = FAppStyle::GetNoBrush();
		if (!InAssetStatusInfoProvider.IsValid())
		{
			return SourceControlStatusOverlay;
		}

		if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			const FString FileName = InAssetStatusInfoProvider->TryGetFilename();
			if (FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(FileName, EStateCacheUsage::Use))
			{
				FSlateIcon SCCIcon = SourceControlState->GetIcon();
				if (SCCIcon.IsSet())
				{
					SourceControlStatusOverlay = SCCIcon.GetOverlayIcon();
				}
			}
		}
		return SourceControlStatusOverlay;
	}

	EVisibility GetSourceControlStatusVisibility(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider)
	{
		EVisibility SourceControlStatusVisibility = EVisibility::Collapsed;
		if (!InAssetStatusInfoProvider.IsValid())
		{
			return SourceControlStatusVisibility;
		}

		if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			const FString FileName = InAssetStatusInfoProvider->TryGetFilename();
			if (FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(FileName, EStateCacheUsage::Use))
			{
				FSlateIcon SCCIcon = SourceControlState->GetIcon();
				if (SCCIcon.IsSet())
				{
					SourceControlStatusVisibility = EVisibility::Visible;
				}
			}
		}
		return SourceControlStatusVisibility;
	}

	FText GetSourceControlStatusDescription(const TSharedPtr<IAssetStatusInfoProvider> InAssetStatusInfoProvider)
	{
		const FText SourceControlDescriptionEmpty = LOCTEXT("NoStatus", "Couldn't retrieve source control status");
		FText SourceControlDescription = FText::GetEmpty();
		if (!InAssetStatusInfoProvider.IsValid())
		{
			return SourceControlDescriptionEmpty;
		}

		if (ISourceControlModule::Get().IsEnabled() && ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			const FString FileName = InAssetStatusInfoProvider->TryGetFilename();
			if (FSourceControlStatePtr SourceControlState = ISourceControlModule::Get().GetProvider().GetState(FileName, EStateCacheUsage::Use))
			{
				// Calling this instead of GetStatusText, since that will check for warnings and won't give out a tooltip for every state unlike GetDisplayTooltip
				SourceControlDescription = SourceControlState->GetDisplayTooltip();
			}
		}
		return SourceControlDescription.IsEmpty() ? SourceControlDescriptionEmpty : SourceControlDescription;
	}
}

FAssetCategoryPath::FAssetCategoryPath(const FText& InCategory)
{
	CategoryPath = { TPair<FName, FCategoryPath>(FName(*FTextInspector::GetSourceString(InCategory)), FCategoryPath(InCategory)) };
}

void FAssetCategoryPath::AppendSubMenu(const FCategoryPath& InSubMenu)
{
	CategoryPath.Add(TPair<FName, FCategoryPath>(FName(*FTextInspector::GetSourceString(InSubMenu.GetSubMenuName())), InSubMenu));
}

FAssetCategoryPath::FAssetCategoryPath(TConstArrayView<FText> InCategoryPath)
{
	check(InCategoryPath.Num() > 0);
	
	for (const FText& CategoryChunk : InCategoryPath)
	{
		CategoryPath.Add(TPair<FName, FCategoryPath>(FName(*FTextInspector::GetSourceString(CategoryChunk)), FCategoryPath(CategoryChunk)));
	}
}

void FAssetCategoryPath::GetSubCategories(TArray<FName>& SubCategories) const
{
	if (HasSubCategory())
	{
		SubCategories.Reserve(SubCategories.Num() + NumSubCategories());
		for (int32 i = 1; i < CategoryPath.Num(); i++)
		{
			SubCategories.Add(CategoryPath[i].Key);
		}
	}
}

void FAssetCategoryPath::GetSubCategoriesText(TArray<FText>& SubCategories) const
{
	if (HasSubCategory())
	{
		SubCategories.Reserve(SubCategories.Num() + NumSubCategories());
		for (int32 i = 1; i < CategoryPath.Num(); i++)
		{
			SubCategories.Add(CategoryPath[i].Value.GetSubMenuName());
		}
	}
}

void FAssetCategoryPath::GetSubCategoriesInfo(TArray<FCategoryPath>& SubCategories) const
{
	if (HasSubCategory())
	{
		SubCategories.Reserve(SubCategories.Num() + NumSubCategories());
		for (int32 i = 1; i < CategoryPath.Num(); i++)
		{
			SubCategories.Add(CategoryPath[i].Value);
		}
	}
}

// UAssetDefinition
//---------------------------------------------------------------------------

UAssetDefinition::UAssetDefinition()
{
}

void UAssetDefinition::PostCDOContruct()
{
	Super::PostCDOContruct();

	if (CanRegisterStatically())
	{
		// The Registry might be null if the module owning the instanced CDO is loaded late into editor lifecycle.
		if (UAssetDefinitionRegistry* AssetDefinitionRegistry = UAssetDefinitionRegistry::Get())
		{
			AssetDefinitionRegistry->RegisterAssetDefinition(this);
		}
	}
}

TConstArrayView<FAssetCategoryPath> UAssetDefinition::GetAssetCategories() const
{
	static const auto Categories = { EAssetCategoryPaths::Misc };
	return Categories;
}

EAssetCommandResult UAssetDefinition::GetSourceFiles(const FAssetSourceFilesArgs& InArgs, TFunctionRef<bool(const FAssetSourceFilesResult& InSourceFile)> SourceFileFunc) const
{
	bool bFoundSomeData = false;

	FString SourceFileTagData;
	FAssetSourceFilesResult Result;
	for (const FAssetData& Asset : InArgs.Assets)
	{
		if (Asset.GetTagValue(UObject::SourceFileTagName(), SourceFileTagData))
		{
			TOptional<FAssetImportInfo> ImportInfoOptional = FAssetImportInfo::FromJson(SourceFileTagData);
			if (ImportInfoOptional.IsSet())
			{
				bFoundSomeData = true;
				FAssetImportInfo& ImportInfo = ImportInfoOptional.GetValue();

				for (FAssetImportInfo::FSourceFile& SourceFiles : ImportInfo.SourceFiles)
				{
					Result.FilePath = MoveTemp(SourceFiles.RelativeFilename);
					Result.DisplayLabel = MoveTemp(SourceFiles.DisplayLabelName);
					Result.Timestamp = MoveTemp(SourceFiles.Timestamp);
					Result.FileHash = MoveTemp(SourceFiles.FileHash);
				
					if (InArgs.FilePathFormat == EPathUse::AbsolutePath)
					{
						Result.FilePath = UAssetImportData::ResolveImportFilename(FStringView(Result.FilePath), Asset.PackageName.ToString());
					}

					if (!SourceFileFunc(Result))
					{
						return EAssetCommandResult::Handled;
					}
				}
			}
		}
	}

	return bFoundSomeData ? EAssetCommandResult::Handled : EAssetCommandResult::Unhandled;
}

EAssetCommandResult UAssetDefinition::GetSourceFiles(const FAssetData& InAsset, TFunctionRef<void(const FAssetImportInfo& AssetImportData)> SourceFileFunc) const
{
	FString SourceFileTagData;
	if (InAsset.GetTagValue(UObject::SourceFileTagName(), SourceFileTagData))
	{
		TOptional<FAssetImportInfo> ImportInfo = FAssetImportInfo::FromJson(SourceFileTagData);
		if (ImportInfo.IsSet())
		{
			SourceFileFunc(ImportInfo.GetValue());
			
			return EAssetCommandResult::Handled;
		}
	}
    
	return EAssetCommandResult::Unhandled;
}

void UAssetDefinition::GetAssetStatusInfo(const TSharedPtr<IAssetStatusInfoProvider>& InAssetStatusInfoProvider, TArray<FAssetDisplayInfo>& OutStatusInfo) const
{
	FAssetDisplayInfo DirtyStatus;
	DirtyStatus.StatusIcon = FAppStyle::GetBrush("ContentBrowser.ContentDirty");
	DirtyStatus.Priority = FAssetStatusPriority(EStatusSeverity::Info, 5);
	DirtyStatus.StatusDescription = LOCTEXT("DirtyAssetTooltip", "Asset has unsaved changes");
	DirtyStatus.IsVisible = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UE::AssetDefinition::Status::GetDirtyStatusVisibility, InAssetStatusInfoProvider));
	OutStatusInfo.Add(DirtyStatus);

	FAssetDisplayInfo SCCStatus;
	SCCStatus.Priority = FAssetStatusPriority(EStatusSeverity::Info, 0);
	SCCStatus.StatusIcon = TAttribute<const FSlateBrush*>::Create(TAttribute<const FSlateBrush*>::FGetter::CreateStatic(&UE::AssetDefinition::Status::GetSourceControlStatusBrush, InAssetStatusInfoProvider));
	SCCStatus.StatusIconOverlay = TAttribute<const FSlateBrush*>::Create(TAttribute<const FSlateBrush*>::FGetter::CreateStatic(&UE::AssetDefinition::Status::GetSourceControlStatusOverlayBrush, InAssetStatusInfoProvider));
	SCCStatus.IsVisible = TAttribute<EVisibility>::Create(TAttribute<EVisibility>::FGetter::CreateStatic(&UE::AssetDefinition::Status::GetSourceControlStatusVisibility, InAssetStatusInfoProvider));
	SCCStatus.StatusDescription = TAttribute<FText>::Create(TAttribute<FText>::FGetter::CreateStatic(&UE::AssetDefinition::Status::GetSourceControlStatusDescription, InAssetStatusInfoProvider));
	OutStatusInfo.Add(SCCStatus);
}

bool UAssetDefinition::CanRegisterStatically() const
{
	return !GetClass()->HasAnyClassFlags(CLASS_Abstract);
}

void UAssetDefinition::BuildFilters(TArray<FAssetFilterData>& OutFilters) const
{
	const TSoftClassPtr<UObject> AssetClassPtr = GetAssetClass();

	if (const UClass* AssetClass = AssetClassPtr.Get())
	{
		// If this asset definition doesn't have any categories it can't have any filters.  Filters need to have a
		// category to be displayed.
		if (GetAssetCategories().Num() == 0)
		{
			return;
		}
		
		// By default we don't advertise filtering if the class is abstract for the asset definition.  Odds are,
		// if they've registered an abstract class as an asset definition, they mean to use it for subclasses.
		if (IncludeClassInFilter == EIncludeClassInFilter::Always || (IncludeClassInFilter == EIncludeClassInFilter::IfClassIsNotAbstract && !AssetClass->HasAnyClassFlags(CLASS_Abstract)))
		{
			FAssetFilterData DefaultFilter;
			DefaultFilter.Name = AssetClassPtr.ToSoftObjectPath().ToString();
			DefaultFilter.DisplayText = GetAssetDisplayName();
			DefaultFilter.FilterCategories = GetAssetCategories();
			DefaultFilter.Filter.ClassPaths.Add(AssetClassPtr.ToSoftObjectPath().GetAssetPath());
			DefaultFilter.Filter.bRecursiveClasses = true;
			OutFilters.Add(MoveTemp(DefaultFilter));
		}
	}
}

#undef LOCTEXT_NAMESPACE
