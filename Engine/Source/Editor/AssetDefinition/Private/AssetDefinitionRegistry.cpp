// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDefinitionRegistry.h"
#include "AssetDefinition.h"
#include "Algo/Accumulate.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "Engine/AssetManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(AssetDefinitionRegistry)

UAssetDefinitionRegistry* UAssetDefinitionRegistry::Singleton = nullptr;
bool UAssetDefinitionRegistry::bHasShutDown = false;

UAssetDefinitionRegistry* UAssetDefinitionRegistry::Get()
{
	if (!Singleton && !bHasShutDown)
	{
		// Required for StartupModule and ShutdownModule to be called and FindModule to list the AssetDefinition module
		FModuleManager::Get().LoadModuleChecked("AssetDefinition");

		Singleton = NewObject<UAssetDefinitionRegistry>();
		Singleton->AddToRoot();
		check(Singleton);
	}

	return Singleton;
}

UAssetDefinitionRegistry::UAssetDefinitionRegistry()
{
	TickerDelegate = FTickerDelegate::CreateUObject(this, &UAssetDefinitionRegistry::TickVersionNotification);
}

void UAssetDefinitionRegistry::BeginDestroy()
{
	if (Singleton == this)
	{
		bHasShutDown = true;
		Singleton = nullptr;
	}

	Super::BeginDestroy();
}

const UAssetDefinition* UAssetDefinitionRegistry::GetAssetDefinitionForAsset(const FAssetData& Asset) const
{
	const UClass* AssetClass = UAssetRegistryHelpers::FindAssetNativeClass(Asset);
	return GetAssetDefinitionForClass(AssetClass);
}

const UAssetDefinition* UAssetDefinitionRegistry::GetAssetDefinitionForClass(const UClass* Class) const
{
	while(Class)
	{
		if (const UAssetDefinition* AssetDefinition = AssetDefinitions.FindRef(Class))
		{
			return AssetDefinition;
		}
		
		Class = Class->GetSuperClass();
	}
	
	return nullptr;
}

TArray<TObjectPtr<UAssetDefinition>> UAssetDefinitionRegistry::GetAllAssetDefinitions() const
{
	TArray<TObjectPtr<UAssetDefinition>> AllAssetDefinitions;
	AssetDefinitions.GenerateValueArray(AllAssetDefinitions);
	return AllAssetDefinitions;
}

TArray<TSoftClassPtr<UObject>> UAssetDefinitionRegistry::GetAllRegisteredAssetClasses() const
{
	TArray<TSoftClassPtr<UObject>> AllRegisteredClasses;
	AssetDefinitions.GenerateKeyArray(AllRegisteredClasses);
	return AllRegisteredClasses;
}

uint64 UAssetDefinitionRegistry::GetAssetDefinitionVersion() const
{
	return Version;
}

void UAssetDefinitionRegistry::RegisterAssetDefinition(UAssetDefinition* AssetDefinition)
{
	check(AssetDefinition);
	
	if (TSoftClassPtr<UObject> SupportedClass = AssetDefinition->GetAssetClass())
	{
		const bool CanAddDefinition =
			!AssetDefinitions.Contains(SupportedClass) ||
			// This hack is basically to ensure that the statically created AssetDefinitions for things like blueprints
			// will work alongside the AssetTypeActions for blueprints until that class can be completely subsumbed.
			(AssetDefinition->CanRegisterStatically() && !AssetDefinitions[SupportedClass]->CanRegisterStatically());
		
		// Normally I'd ensure here, we ignore duplicates, but the IAssetTypeActions allowed duplicates.
		if (CanAddDefinition)
		{
			AssetDefinitions.Add(SupportedClass, AssetDefinition);

			Version++;
			RegisterTickerForVersionNotification();
		}
		else
		{
			UAssetDefinition* ExistingDefinition = AssetDefinitions[SupportedClass];
			//UE_LOGF(LogTemp, Warning, "Unable to register %ls for (%ls), %ls is already registered",  *GetPathNameSafe(AssetDefinition), *SupportedClass.ToString(), *GetPathNameSafe(ExistingDefinition));
		}
	}
}

void UAssetDefinitionRegistry::UnregisterAssetDefinition(UAssetDefinition* AssetDefinition)
{
	check(AssetDefinition);
	
	if (TSoftClassPtr<UObject> SupportedClass = AssetDefinition->GetAssetClass())
	{
		AssetDefinitions.Remove(SupportedClass);
		
		Version++;
		RegisterTickerForVersionNotification();
	}
}

FOnAssetDefinitionRegistryVersionChange& UAssetDefinitionRegistry::OnAssetDefinitionRegistryVersionChange()
{
	return OnAssetDefinitionRegistryVersionChangeDelegate;
}

void UAssetDefinitionRegistry::RegisterTickerForVersionNotification()
{
	if (!TickerDelegateHandle.IsValid())
	{
		TickerDelegateHandle = FTSTicker::GetCoreTicker().AddTicker(TickerDelegate);
	}
}

bool UAssetDefinitionRegistry::TickVersionNotification(float)
{
	OnAssetDefinitionRegistryVersionChangeDelegate.Broadcast(this);
	TickerDelegateHandle.Reset();
	return false; // One-shot
}
