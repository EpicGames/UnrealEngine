// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetStatusAssetDataInfoProvider.h"

#include "AssetRegistry/AssetData.h"
#include "Misc/PackageName.h"
#include "UObject/UObjectGlobals.h"

UPackage* FAssetStatusAssetDataInfoProvider::FindPackage() const
{
	return FindObjectSafe<UPackage>(NULL, *FNameBuilder(AssetData.PackageName), EFindObjectFlags::ExactClass);
}

FString FAssetStatusAssetDataInfoProvider::TryGetFilename() const
{
	FString OutFileName;
	const FString* PackageExtension = AssetData.HasAnyPackageFlags(PKG_ContainsMap) ? &FPackageName::GetMapPackageExtension() : &FPackageName::GetAssetPackageExtension();
	FPackageName::TryConvertLongPackageNameToFilename(AssetData.PackageName.ToString(), OutFileName, *PackageExtension);
	return OutFileName;
}

FAssetData FAssetStatusAssetDataInfoProvider::TryGetAssetData() const
{
	return AssetData.IsValid() ? AssetData : FAssetData();
}
