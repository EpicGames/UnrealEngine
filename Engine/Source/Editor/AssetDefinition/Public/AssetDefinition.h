// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"
#include "AssetRegistry/AssetData.h"
#include "Framework/SlateDelegates.h"
#include "Misc/ScopedSlowTask.h"
#include "Toolkits/IToolkit.h"
#include "Misc/AssetFilterData.h"
#include "Widgets/Input/SButton.h"

#include "AssetDefinition.generated.h"

#define UE_API ASSETDEFINITION_API

class IToolkitHost;
class UThumbnailInfo;
struct FAssetDisplayInfo;
struct FSlateBrush;
class IAssetSystemInfoProvider;
class IAssetStatusInfoProvider;
class ISourceControlRevision;
class SWidget;

UENUM()
enum class EAssetActivationMethod : uint8
{
	DoubleClicked,
	Opened,
	Previewed
};

UENUM()
enum class EAssetCommandResult : uint8
{
	Handled,
	Unhandled
};

UENUM()
enum class EAssetOpenMethod : uint8
{
	Edit,
	View,
	//Preview
};

enum class EAssetMergeResult : uint8
{
	Unknown,
	Completed,
	Cancelled,
};

enum class EPathUse : uint8
{
	/**
	 * Present the file in a UI friendly format. 
	 * Note that the path might not always be relative.
	 * It cannot be used for other purposes then presenting it to the user since the details of how to resolve that path might change based on the asset implementation.
	 */
	Display,

	/**
	 * Resolve the path into an absolute path.
	 */
	AbsolutePath
};

struct FAssetArgs
{
	FAssetArgs() { }
	FAssetArgs(TConstArrayView<FAssetData> InAssets) : Assets(InAssets) { }
	
	TConstArrayView<FAssetData> Assets;

	template<typename ExpectedObjectType>
	TArray<ExpectedObjectType*> LoadObjects(const TSet<FName>& LoadTags = {}, TArray<FAssetData>* OutAssetsThatFailedToLoad = nullptr) const
	{
		FScopedSlowTask SlowTask((float)Assets.Num());
	
		TArray<ExpectedObjectType*> LoadedObjects;
		LoadedObjects.Reserve(Assets.Num());
		
		for (const FAssetData& Asset : Assets)
		{
			SlowTask.EnterProgressFrame(1, FText::FromString(Asset.GetObjectPathString()));
			
			if (Asset.IsInstanceOf(ExpectedObjectType::StaticClass()))
			{
				if (ExpectedObjectType* ExpectedType = Cast<ExpectedObjectType>(Asset.GetAsset(LoadTags)))
				{
					LoadedObjects.Add(ExpectedType);
					continue;
				}
			}

			// If we get here than we failed to load the asset for some reason or another.
			if (OutAssetsThatFailedToLoad)
			{
				OutAssetsThatFailedToLoad->Add(Asset);
			}
		}
		
		return LoadedObjects;
	}
	
	template<typename ExpectedObjectType>
    ExpectedObjectType* LoadFirstValid(const TSet<FName>& LoadTags = {}) const
    {   	
    	for (const FAssetData& Asset : Assets)
    	{
    		if (Asset.IsInstanceOf(ExpectedObjectType::StaticClass()))
    		{
    			if (ExpectedObjectType* ExpectedType = Cast<ExpectedObjectType>(Asset.GetAsset(LoadTags)))
    			{
    				return ExpectedType;
    			}
    		}	
    	}
    	
    	return nullptr;
    }
};

struct FAssetOpenArgs : public FAssetArgs
{
	EAssetOpenMethod OpenMethod;
	TSharedPtr<IToolkitHost> ToolkitHost;

	EToolkitMode::Type GetToolkitMode() const { return ToolkitHost.IsValid() ? EToolkitMode::WorldCentric : EToolkitMode::Standalone; }
};

struct FAssetActivateArgs : public FAssetArgs
{
	EAssetActivationMethod ActivationMethod;
};

struct FAssetSourceFilesArgs : public FAssetArgs
{
	EPathUse FilePathFormat = EPathUse::AbsolutePath;
};

struct FAssetSourceFilesResult
{
	/** The file path in the format requested. */
	FString FilePath;

	/** The Label was used to display this source file in the property editor. */
	FString DisplayLabel;

	/** The timestamp of the file when it was imported (as UTC). 0 when unknown. */
	FDateTime Timestamp;

	/** The MD5 hash of the file when it was imported. Invalid when unknown. */
	FMD5Hash FileHash;
};

struct FAssetMergeResults
{
	UPackage* MergedPackage = nullptr;
	EAssetMergeResult Result = EAssetMergeResult::Unknown;
};

DECLARE_DELEGATE_OneParam(FOnAssetMergeResolved, const FAssetMergeResults& Results);

enum EMergeFlags : uint8
{
	MF_NONE                    = 0x00,
	MF_NO_GUI                  = 0x01,
	MF_HANDLE_SOURCE_CONTROL   = 0x02,
};

struct FAssetAutomaticMergeArgs
{
	UObject* LocalAsset = nullptr;
	FOnAssetMergeResolved ResolutionCallback;
	EMergeFlags Flags = MF_HANDLE_SOURCE_CONTROL;
};

struct FAssetManualMergeArgs
{
	UObject* LocalAsset = nullptr;
	UObject* BaseAsset = nullptr;
	UObject* RemoteAsset = nullptr;
	FOnAssetMergeResolved ResolutionCallback;
	EMergeFlags Flags = MF_HANDLE_SOURCE_CONTROL;
};

struct FAssetSupportResponse
{
public:
	static FAssetSupportResponse Supported()
	{
		return FAssetSupportResponse(true, FText::GetEmpty());
	}

	static FAssetSupportResponse NotSupported()
	{
		return FAssetSupportResponse(false, FText::GetEmpty());
	}

	static FAssetSupportResponse Error(const FText& ErrorText)
	{
		return FAssetSupportResponse(false, ErrorText);
	}

	bool IsSupported() const { return bSupported; }
	const FText& GetErrorText() const { return ErrorText; }

private:
	FAssetSupportResponse(bool InSupported, const FText InError)
		: bSupported(InSupported)
		, ErrorText(InError)
	{
	}

private:
	bool bSupported;
	FText ErrorText;
};

/* Revision information for a single revision of a file in source control */
USTRUCT(BlueprintType)
struct FRevisionInfo
{
	GENERATED_BODY()

	UPROPERTY(BlueprintReadWrite, Category="Asset Revision")
	FString		Revision;

	UPROPERTY(BlueprintReadWrite, Category="Asset Revision")
	int32		Changelist = -1;

	UPROPERTY(BlueprintReadWrite, Category="Asset Revision")
	FDateTime	Date;

	static inline FRevisionInfo InvalidRevision()
	{
		static const FRevisionInfo Ret = { TEXT(""), -1, FDateTime() };
		return Ret;
	}
};

struct FAssetDiffArgs
{
	UObject* OldAsset = nullptr;
	FRevisionInfo OldRevision;

	UObject* NewAsset = nullptr;
	FRevisionInfo NewRevision;
};

struct FAssetOpenSupportArgs
{
	FAssetOpenSupportArgs(EAssetOpenMethod InOpenMethod = EAssetOpenMethod::Edit)
		: OpenMethod(InOpenMethod)
	{
		
	}
	
	EAssetOpenMethod OpenMethod;
};


/**
 * These are just some common asset categories.  You're not at all limited to these, and can register an "Advanced"
 * category with the IAssetTools::RegisterAdvancedAssetCategory.
 */
struct EAssetCategoryPaths
{
	// This category is special, "Basic" assets appear at the very top level and are not placed into any submenu.
	// Arguably the basic category should not exist and should instead be user configurable on what they feel should be
	// top level assets.
	static UE_API FAssetCategoryPath Basic;
	
	static UE_API FAssetCategoryPath Animation;
	static UE_API FAssetCategoryPath Audio;
	static UE_API FAssetCategoryPath Blueprint;
	static UE_API FAssetCategoryPath Cinematics;
	static UE_API FAssetCategoryPath Data;
	static UE_API FAssetCategoryPath Foliage;
	static UE_API FAssetCategoryPath FX;
	static UE_API FAssetCategoryPath Gameplay;
	static UE_API FAssetCategoryPath AI;
	static UE_API FAssetCategoryPath Input;
	static UE_API FAssetCategoryPath Material;
	static UE_API FAssetCategoryPath Media;
	static UE_API FAssetCategoryPath Misc;
	static UE_API FAssetCategoryPath Physics;
	static UE_API FAssetCategoryPath Texture;
	static UE_API FAssetCategoryPath UI;
	static UE_API FAssetCategoryPath World;
};

struct FAssetOpenSupport
{
public:
	FAssetOpenSupport(EAssetOpenMethod InOpenMethod, bool bInSupported)
		: OpenMethod(InOpenMethod)
		, IsSupported(bInSupported)
	{
	}
	
	FAssetOpenSupport(EAssetOpenMethod InOpenMethod, bool bInSupported, EToolkitMode::Type InRequiredToolkitMode)
		: OpenMethod(InOpenMethod)
		, IsSupported(bInSupported)
		, RequiredToolkitMode(InRequiredToolkitMode)
	{
	}
	
	EAssetOpenMethod OpenMethod;
	bool IsSupported;
	TOptional<EToolkitMode::Type> RequiredToolkitMode;
};

class UAssetDefinitionRegistry;
struct FAssetImportInfo;

enum class EIncludeClassInFilter : uint8
{
	IfClassIsNotAbstract,
	Always
};


struct FAssetFilterDataCache
{
public:
	TArray<FAssetFilterData> Filters;
};

DECLARE_DELEGATE_RetVal(bool, FIsActionPlaying)
struct FAssetActionThumbnailOverlayInfo
{
public:
	UE_DEPRECATED(5.8, "Please use the IsActionPlayingDelegate instead.")
	/** Image widget displayed in the top left border */
	TSharedPtr<SWidget> ActionImageWidget;

	/** Attach your IsPlaying action to this delegate that will be used to decide on the action image to use */
	FIsActionPlaying IsActionPlayingDelegate;

	/** Button Args used to construct the Action Button */
	SButton::FArguments ActionButtonArgs;
};

struct FAssetButtonActionExtension
{
public:
	TAttribute<FText> PickTooltipAttribute;
	TAttribute<const FSlateBrush*> PickBrushAttribute;
	FOnClicked OnClicked;
};

/**
 * Asset Definitions represent top level assets that are known to the editor.
 *
 * -- Conversion Guide --
 * Asset Definitions (UAssetDefinitionDefault) are a replacement to the Asset Actions (FAssetTypeActions_Base) system.
 * The reasons for the replacement are multitude, but the highlights are,
 * 
 * Asset Definitions no longer do things like provide a GetActions function, the replacement for this is you using the
 * new UToolMenu extension system to register actions for the assets, an example is in this document.  A lot of the
 * APIs have been tweaked and cleaned up to make them easier to upgrade in the future, a lot of the original API for
 * Asset Actions were added one at a time, and several of them could be combined in simpler ways.  Another benefit is
 * soon we might be able to make the AssetTools module no longer a circular dependency.
 * 
 * All of this is in service to what was previously impossible.  To be able to right click on assets in the Content
 * Browser and NOT have the asset and every asset it knew about load on right click, this previous impossible to escape
 * byproduct slowed down working in the editor constantly because things that didn’t require opening the asset became
 * necessary, and some assets could load *A LOT* of other content.
 * 
 * Unfortunately I can’t prevent people from backsliding, at least for now.  Even after fixing the APIs to not require
 * loading, people need to be cleverer (Use Asset Tag Data) in how they provide right click options for assets.  But to
 * help, you can run the editor with -WarnIfAssetsLoaded on the command line.  I’ve added a new utility class to the
 * engine called, FWarnIfAssetsLoadedInScope, it causes notifications with callstacks to be popped up telling you what
 * code is actually responsible for any asset loads within earmarked scopes that should NOT be loading assets.
 * 
 * Backwards Compatibility
 * The new system is fully* backwards compatible.  Asset Definitions are registered with the old Asset Tools
 * (IAssetTools::RegisterAssetTypeActions) this is done through the   FAssetDefinitionProxy.  Similarly, Asset Actions
 * (FAssetTypeActions_Base) are registered with the Asset Definition Registry with an Asset Definition Proxy (UAssetDefinition_AssetTypeActionsProxy).
 * 
 * When converting Asset Actions to AssetDefinitions and you’re trying to understand how to map a specific function to
 * the new system it can be helpful to look at the equivalent function in FAssetDefinitionProxy.
 * 
 * IMPORTANT - You are no longer allowed to register multiple Asset Definitions for the same Asset Class.  There were a
 * very small number of people doing this to do some tricky things with filters which are no longer required.  The new
 * system will yell at you if you do this.
 * 
 * 
 * Registration
 * Registering your Asset Definition is no longer required like it was for Asset Actions.  The UObjects are automatically
 * registered with the new Asset Definition Registry (UAssetDefinitionRegistry).
 * 
 * You no longer need to register Categories for your Asset Definition, like you had to do with Asset Actions.  Your
 * Assets categories are just an array of FAssetCategoryPath.  They have accelerator constructors to just take an FText
 * for the main category, and the sub category (if there is one) which replaces the whole “GetSubMenus” function from
 * Asset Actions.  The new version can go further, with multiple sub menus and categories, but the UI isn’t set up for it yet.
 * 
 * GetActions
 * The function GetActions no longer exists, the replacement is to somewhere put a self registering callback to register
 * the UToolMenu extension, but you can just put it at the bottom of your .cpp for your AssetDefinition, that is where
 * the others are.  The template goes something like this,
 * 
 * // Menu Extensions
 * //--------------------------------------------------------------------
 * 
 * 
 * namespace MenuExtension_YOUR_CLASS_TYPE
 * {
 *   static void ExecuteActionForYourType(const FToolMenuContext& InContext)
 *   {
 *      const UContentBrowserAssetContextMenuContext* Context = UContentBrowserAssetContextMenuContext::FindContextWithAssets(InContext);
 *     
 *      for (YOUR_CLASS_TYPE* LoadedInstance : Context->LoadSelectedObjects<YOUR_CLASS_TYPE>())
 *      {
 *      }
 *   }
 *  
 *   static FDelayedAutoRegisterHelper DelayedAutoRegister(EDelayedRegisterRunPhase::EndOfEngineInit, []{
 *      UToolMenus::RegisterStartupCallback(FSimpleMulticastDelegate::FDelegate::CreateLambda([]()
 *      {
 *         FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
 *         UToolMenu* Menu = UE::ContentBrowser::ExtendToolMenu_AssetContextMenu(YOUR_CLASS_TYPE::StaticClass());
 *     
 *         FToolMenuSection& Section = Menu->FindOrAddSection("GetAssetActions");
 *         Section.AddDynamicEntry(NAME_None, FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
 *         {
 *            const TAttribute<FText> Label = LOCTEXT("YOUR_ASSET_TYPE_ExecuteActionForYourType", "Execute Action For Your Type");
 *            const TAttribute<FText> ToolTip = LOCTEXT("YOUR_ASSET_TYPE_ExecuteActionForYourTypeTooltip", "This will execute the action for your type.");
 *            const FSlateIcon Icon = FSlateIcon();
 * 
 * 
 *            FToolUIAction UIAction;
 *            UIAction.ExecuteAction = FToolMenuExecuteAction::CreateStatic(&ExecuteActionForYourType);
 *            InSection.AddMenuEntry("YOUR_ASSET_TYPE_ExecuteActionForYourType", Label, ToolTip, Icon, UIAction);
 *         ));
 *      }));
 *   });
 * }
 * 
 * It’s very important that you do not load the asset in your CanExecuteAction callback or in this self callback, you should save that until you finally get Executed.
 * If you’re looking for examples, there are tons you'll find by searching for “namespace MenuExtension_”.
 * 
 * GetFilterName & BuildBackendFilter
 * These functions have been replaced by BuildFilters. The new function is great, you can provide an array of filters that are available with this asset.  So for example,
 * Blueprints provide a filter for Blueprint Class, but they also provide the filters for Blueprint Interface, Blueprint Macro Library and Blueprint Function library,
 * which are all UBlueprint assets, but differ based on Asset Tag data.
 */
UCLASS(MinimalAPI, Abstract)
class UAssetDefinition : public UObject
{
	GENERATED_BODY()

public:
	UE_API UAssetDefinition();

	//Begin UObject
	UE_API virtual void PostCDOContruct() override;
	//End UObject
	
public:
	
	/** Returns the name of this type */
	virtual FText GetAssetDisplayName() const PURE_VIRTUAL(UAssetDefinition::GetAssetDisplayName, (void)ensureMsgf(false && "NotImplemented", TEXT("NotImplemented")); return FText(); );

	/**
	 * Returns the name of this type, but allows overriding the default on a specific instance of the asset.  This
	 * is handy for cases like UAssetData which are of course all UAssetData, but a given instance of the asset
	 * is really a specific instance of some UAssetData class, and being able to override that on the instance is handy
	 * for readability at the Content Browser level.
	 */
	virtual FText GetAssetDisplayName(const FAssetData& AssetData) const { return GetAssetDisplayName(); }

	/** Get the supported class of this type. */
	virtual TSoftClassPtr<UObject> GetAssetClass() const PURE_VIRTUAL(UAssetDefinition::GetAssetClass, (void)ensureMsgf(false && "NotImplemented", TEXT("NotImplemented")); return TSoftClassPtr<UClass>(); );

	/** Returns the color associated with this type */
	virtual FLinearColor GetAssetColor() const PURE_VIRTUAL(UAssetDefinition::GetAssetColor, (void)ensureMsgf(false && "NotImplemented", TEXT("NotImplemented")); return FColor::Red; );

	/** Returns additional tooltip information for the specified asset, if it has any. */
	virtual FText GetAssetDescription(const FAssetData& AssetData) const { return FText::GetEmpty(); }

	/** Gets a list of categories this asset is in, these categories are used to help organize */
	UE_API virtual TConstArrayView<FAssetCategoryPath> GetAssetCategories() const;
	
public:
	// Common Operations
	virtual TArray<FAssetData> PrepareToActivateAssets(const FAssetActivateArgs& ActivateArgs) const
	{
		return TArray<FAssetData>(ActivateArgs.Assets);
	}
	
	virtual bool ShouldFindEditorForAsset() const
	{ 
		return true;
	}

	/** Get open support for the method.  Includes required information before we call OpenAsset. */
	virtual FAssetOpenSupport GetAssetOpenSupport(const FAssetOpenSupportArgs& OpenSupportArgs) const
	{
		return FAssetOpenSupport(OpenSupportArgs.OpenMethod,OpenSupportArgs.OpenMethod == EAssetOpenMethod::Edit); 
	}
	
	virtual EAssetCommandResult OpenAssets(const FAssetOpenArgs& OpenArgs) const PURE_VIRTUAL(UAssetDefinition::OpenAsset, (void)ensureMsgf(false && "NotImplemented", TEXT("NotImplemented")); return EAssetCommandResult::Unhandled; );

	virtual EAssetCommandResult ActivateAssets(const FAssetActivateArgs& ActivateArgs) const 
	{
		return EAssetCommandResult::Unhandled;
	}

	
	// Common Queries
	virtual FAssetSupportResponse CanRename(const FAssetData& InAsset) const
	{
		return FAssetSupportResponse::Supported();
	}

	virtual FAssetSupportResponse CanDuplicate(const FAssetData& InAsset) const
	{
		return FAssetSupportResponse::Supported();
	}

	virtual FAssetSupportResponse CanLocalize(const FAssetData& InAsset) const
	{
		return FAssetSupportResponse::Supported();
	}
	
	
	// Importing
	virtual bool CanImport() const { return false; }

	
	// Merging
	virtual bool CanMerge() const { return false; }
	virtual EAssetCommandResult Merge(const FAssetAutomaticMergeArgs& MergeArgs) const { return EAssetCommandResult::Unhandled; }
	virtual EAssetCommandResult Merge(const FAssetManualMergeArgs& MergeArgs) const { return EAssetCommandResult::Unhandled; }


	// Filtering
	virtual TSharedRef<FAssetFilterDataCache> GetFilters() const
	{
		if (!FilterCache.IsValid())
		{
			FilterCache = MakeShared<FAssetFilterDataCache>();
			BuildFilters(FilterCache->Filters);
		}

		return FilterCache.ToSharedRef();
	}

protected:
	UE_API virtual void BuildFilters(TArray<FAssetFilterData>& OutFilters) const;

public:
	// Extras
	virtual FText GetObjectDisplayNameText(UObject* Object) const
	{
		return FText::FromString(Object->GetName());
	}

	// Source Files

	/**
	 * Return the source files that was used to generate/import the asset
	 * @param InArgs The asset data of the assets we want the source files from and in which format we want the file path to be.
	 * @param SourceFileFunc A function that is called for each source file found. The call back must return true to continue the enumeration.
	 */
	UE_API virtual EAssetCommandResult GetSourceFiles(const FAssetSourceFilesArgs& InArgs, TFunctionRef<bool(const FAssetSourceFilesResult& InSourceFile)> SourceFileFunc) const;

	UE_DEPRECATED(5.3, "This override will be removed because it doesn't account that the resolution of a relative path to its absolute path may varies based on the asset implementation.")
	UE_API virtual EAssetCommandResult GetSourceFiles(const FAssetData& InAsset, TFunctionRef<void(const FAssetImportInfo& AssetImportData)> SourceFileFunc) const;

	// Diffing Assets
	virtual EAssetCommandResult PerformAssetDiff(const FAssetDiffArgs& DiffArgs) const
	{
		return EAssetCommandResult::Unhandled;
	}

	// Thumbnails

	/** Returns the thumbnail info for the specified asset, if it has one. This typically requires loading the asset.  */
	virtual UThumbnailInfo* LoadThumbnailInfo(const FAssetData& InAssetData) const
	{
		return nullptr;
	}

	/** Returns thumbnail brush unique for the given asset data.  Returning null falls back to class thumbnail brush. */
	virtual const FSlateBrush* GetThumbnailBrush(const FAssetData& InAssetData, const FName InClassName) const
	{
		return nullptr;
	}

	/** Returns icon brush unique for the given asset data.  Returning null falls back to class icon brush. */
	virtual const FSlateBrush* GetIconBrush(const FAssetData& InAssetData, const FName InClassName) const
	{
		return nullptr;
	}

	UE_DEPRECATED(5.5, "Please use the new GetThumbnailActionOverlay function")
	/** Optionally returns a custom widget to overlay on top of this assets' thumbnail */
	virtual TSharedPtr<SWidget> GetThumbnailOverlay(const FAssetData& InAssetData) const
	{
		return TSharedPtr<SWidget>();
	}

	virtual bool GetThumbnailActionOverlay(const FAssetData& InAssetData, FAssetActionThumbnailOverlayInfo& OutActionOverlayInfo) const
	{
		return false;
	}

	/** Get the basic status info on assets like the Dirty and SourceControl one */
	UE_API virtual void GetAssetStatusInfo(const TSharedPtr<IAssetStatusInfoProvider>& InAssetStatusInfoProvider, TArray<FAssetDisplayInfo>& OutStatusInfo) const;

	/**Optionally adds custom Asset Picker buttons to the Property Editor for the selected AssetDefinition, using the provided extension data this function returns.*/
	virtual void GetAssetActionButtonExtensions(const FAssetData& InAssetData, TArray<FAssetButtonActionExtension>& OutExtensions) const { };

	/** Whether this asset has external packages associated with it (impacts saving and dirty-state behavior) */
	virtual bool ShouldSaveExternalPackages() const
	{
		return false;
	}	

	// DEVELOPER NOTE:
	// Originally this class was based on the IAssetTypeActions implementation.  Several of the functions on there
	// were created organically and added without a larger discussion about if such a thing belonged on those classes.
	//
	// For example, IAssetTypeActions::ShouldForceWorldCentric was needed for a single asset, but we didn't instead
	// implement GetAssetOpenSupport, which merges the needs of ShouldForceWorldCentric, and SupportsOpenedMethod.
	// 
	// Another example, is IAssetTypeActions::SetSupported and IAssetTypeActions::IsSupported.  These were concepts
	// that could have lived in a map on the registry and never needed to be stored on the actual IAssetTypeActions.
	//
	// So, please do not add new functions to this class if it can be helped.  The AssetDefinitions are intended to be
	// a basic low level representation of top level assets for the Content Browser and other editor tools to do
	// some basic interaction with them, or learn some basic common details about them.
	//
	// If you must add a new function to this class, some requests,
	// 1. Can it be added as a parameter to an existing Argument struct for an existing function?  If so, please do that.
	// 2. Can it be added as part of the return structure of an existing function?  If so, please do that.
	// 3. If you add a new function, please create a struct for the Args.  We'll be able to upgrade things easier.
	//    Please continue to use EAssetCommandResult and FAssetSupportResponse, for those kinds of commands.

	UE_API virtual bool CanRegisterStatically() const;

protected:
	EIncludeClassInFilter IncludeClassInFilter = EIncludeClassInFilter::IfClassIsNotAbstract;

private:
	mutable TSharedPtr<FAssetFilterDataCache> FilterCache;

	friend class UAssetDefinitionRegistry;
};

#undef UE_API
