// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Internationalization/Text.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"

struct FSlateBrush;

enum class EStatusSeverity
{
	Info,
	Warning,
	Error
};
ENUM_CLASS_FLAGS(EStatusSeverity)

struct FAssetStatusPriority
{
	FAssetStatusPriority()
		: Severity(EStatusSeverity::Info)
		, SeverityPriority(0)
	{}

	FAssetStatusPriority(const TAttribute<EStatusSeverity>& InSeverity)
		: Severity(InSeverity)
		, SeverityPriority(0)
	{}

	FAssetStatusPriority(const TAttribute<EStatusSeverity>& InSeverity, int32 InSeverityPriority)
		: Severity(InSeverity)
		, SeverityPriority(InSeverityPriority)
	{}

public:
	bool operator==(const FAssetStatusPriority& InOtherStatusPriority) const
	{
		if (!Severity.IsSet() || !InOtherStatusPriority.Severity.IsSet())
		{
			return false;
		}

		return Severity.Get() == InOtherStatusPriority.Severity.Get() && SeverityPriority == InOtherStatusPriority.SeverityPriority;
	}

	bool operator<(const FAssetStatusPriority& InOtherStatusPriority) const
	{
		if (!Severity.IsSet())
		{
			return true;
		}

		if (!InOtherStatusPriority.Severity.IsSet())
		{
			return false;
		}

		if (Severity.Get() == InOtherStatusPriority.Severity.Get())
		{
			return SeverityPriority < InOtherStatusPriority.SeverityPriority;
		}

		return Severity.Get() < InOtherStatusPriority.Severity.Get();
	}

public:
	TAttribute<EStatusSeverity> Severity;
	int32 SeverityPriority;
};

struct FAssetDisplayInfo
{
public:
	TAttribute<const FSlateBrush*> StatusIcon;
	TAttribute<const FSlateBrush*> StatusIconOverlay;
	TAttribute<FText> StatusTitle;
	TAttribute<FText> StatusDescription;
	TAttribute<EVisibility> IsVisible;
	TAttribute<FAssetStatusPriority> Priority;
};
