// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Engine/EngineBaseTypes.h"
#include "UObject/Object.h"

#include "AssetDefinitionRegistry.generated.h"

#define UE_API ASSETDEFINITION_API

class UAssetDefinition;

DECLARE_MULTICAST_DELEGATE_OneParam(FOnAssetDefinitionRegistryVersionChange, UAssetDefinitionRegistry*);

UCLASS(MinimalAPI, config=Editor)
class UAssetDefinitionRegistry : public UObject
{
	GENERATED_BODY()

public:	
	static UE_API UAssetDefinitionRegistry* Get();

	UE_API UAssetDefinitionRegistry();
	
	UE_API virtual void BeginDestroy() override;

	UE_API const UAssetDefinition* GetAssetDefinitionForAsset(const FAssetData& Asset) const;
	UE_API const UAssetDefinition* GetAssetDefinitionForClass(const UClass* Class) const;

	// Gets the current version of the AssetDefinitions.   Version is updated whenever an AssetDefinition is Registered/Unregistered
	UE_API uint64 GetAssetDefinitionVersion() const;
	UE_API TArray<TObjectPtr<UAssetDefinition>> GetAllAssetDefinitions() const;
	UE_API TArray<TSoftClassPtr<UObject>> GetAllRegisteredAssetClasses() const;

	/**
	 * Normally UAssetDefinitionRegistry are registered automatically by their CDO.  The only reason you need to do this is if
	 * you're forced to dynamically create the UAssetDefinition at runtime.  The original reason for this function was
	 * to be able to create wrappers for the to be replaced IAssetTypeActions, that you can access AssetDefinition
	 * versions of any IAssetType making the upgrade easier.
	 */
	UE_API void RegisterAssetDefinition(UAssetDefinition* AssetDefinition);

	UE_API void UnregisterAssetDefinition(UAssetDefinition* AssetDefinition);

	/**
	 * Called when the AssetDefinitionRegistry's version has changed.
	 */
	UE_API FOnAssetDefinitionRegistryVersionChange& OnAssetDefinitionRegistryVersionChange();
	
private:
	UE_API void RegisterTickerForVersionNotification();
	UE_API bool TickVersionNotification(float);
	
	static UE_API UAssetDefinitionRegistry* Singleton;
	static UE_API bool bHasShutDown;

	UPROPERTY()
	TMap<TSoftClassPtr<UObject>, TObjectPtr<UAssetDefinition>> AssetDefinitions;

	uint64 Version;

	FTickerDelegate TickerDelegate;
	FTSTicker::FDelegateHandle TickerDelegateHandle;
	FOnAssetDefinitionRegistryVersionChange OnAssetDefinitionRegistryVersionChangeDelegate;
};

#undef UE_API
