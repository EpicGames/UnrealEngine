// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IAssetStatusInfoProvider.h"
#include "AssetRegistry/AssetData.h"

#define UE_API ASSETDEFINITION_API

class FAssetStatusAssetDataInfoProvider : public IAssetStatusInfoProvider
{
public:
	FAssetStatusAssetDataInfoProvider(FAssetData InAssetData)
		: AssetData(InAssetData)
	{}

	UE_API virtual UPackage* FindPackage() const override;

	UE_API virtual FString TryGetFilename() const override;

	UE_API virtual FAssetData TryGetAssetData() const override;

private:
	FAssetData AssetData;
};

#undef UE_API
