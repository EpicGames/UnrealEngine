// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/AssetData.h"
#include "Containers/UnrealString.h"

class UPackage;

class IAssetStatusInfoProvider
{
public:
	virtual ~IAssetStatusInfoProvider() = default;

	/** Try to find the Package without loading it, if the package is not loaded it will return nullptr */
	virtual UPackage* FindPackage() const = 0;

	/** Try to get the filename, return empty otherwise */
	virtual FString TryGetFilename() const = 0;

	/** Try to get the AssetData, return an empty FAssetData otherwise */
	virtual FAssetData TryGetAssetData() const = 0;
};
