// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"

struct FAssetDisplayInfo;

class IAssetSystemInfoProvider
{
public:
	virtual ~IAssetSystemInfoProvider() = default;

	/** Populate the OutAssetDisplayInfo with the information of the Asset like the Path/Size etc... */
	virtual void PopulateAssetInfo(TArray<FAssetDisplayInfo>& OutAssetDisplayInfo) const = 0;
};
