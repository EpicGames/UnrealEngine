// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#define UE_API ASSETDEFINITION_API

/** 
 * Whether the SubCategory should be a menu or just a section
 */
enum class ECategoryMenuType : uint8
{
	Menu = 0,
	Section = 1 << 0
};

/**
 * Contains the sub menu name and the type of the sub menu, either a menu or a section, section should be the last in the chain
 */
struct FCategoryPath
{
	FCategoryPath(const FText& InSubMenuName, const ECategoryMenuType InCategoryMenuType = ECategoryMenuType::Menu)
		: SubMenuName(InSubMenuName)
		, CategoryMenuType(InCategoryMenuType)
	{}

	/** Get the name of the sub menu, could be a sub category based on the position inside the FAssetCategoryPath */
	FText GetSubMenuName() const { return SubMenuName; }

	/** Either Menu or Section, based on if you want a menu or a section for your entry */
	ECategoryMenuType GetCategoryMenuType() const { return CategoryMenuType; }

private:
	FText SubMenuName;
	ECategoryMenuType CategoryMenuType;
};

/**
 * The asset category path is how we know how to build menus around assets.\n  For example, Basic is generally the ones
 * we expose at the top level, where as everything else is a category with a pull out menu, and the subcategory would
 * be where it gets placed in a submenu inside of there.\n\n
 * Currently, the system supports chaining menus and as the last entry you can have a custom section.\n
 * You can't have a menu inside a section so the section should be the last in case you want one.
 */
struct FAssetCategoryPath
{
	UE_API FAssetCategoryPath(const FText& InCategory);
	UE_API FAssetCategoryPath(TConstArrayView<FText> InCategoryPath);

	/** 
	 * Constructs from a root FAssetCategoryPath followed by any number of FCategoryPath.\n
	 * By default, every FText given is treated like a menu.\n
	 * If the last entry is a section you should still explicitly add it like so:\n
	 * FAssetCategoryPath(FAssetCategoryPath or (FText), FCategoryPath or (FText)..., FCategoryPath(FText, ECategoryMenuType::Section))
	 */
	template <typename... TInSubMenus>
	requires (std::is_constructible_v<FCategoryPath, TInSubMenus> && ...)
	FAssetCategoryPath(const FAssetCategoryPath& InCategory, TInSubMenus&&... InSubMenus)
	{
		// Append the first category
		CategoryPath.Append(InCategory.CategoryPath);

		// Append all the additional sub-menus/sections
		(AppendSubMenu(FCategoryPath(Forward<TInSubMenus>(InSubMenus))), ...);
	}

	/** Constructor helper to avoid explicit wrapping when a specific menu type is needed */
	FAssetCategoryPath(const FAssetCategoryPath& InCategory, const FText& InSubMenuName, ECategoryMenuType InMenuType = ECategoryMenuType::Menu)
		: FAssetCategoryPath(InCategory, FCategoryPath(InSubMenuName, InMenuType))
	{}

	FName GetCategory() const { return CategoryPath[0].Key; }
	FText GetCategoryText() const { return CategoryPath[0].Value.GetSubMenuName(); }
	
	bool HasSubCategory() const { return CategoryPath.Num() > 1; }
	int32 NumSubCategories() const { return HasSubCategory() ? (CategoryPath.Num() - 1) : 0; }
	FName GetSubCategory() const { return HasSubCategory() ? CategoryPath[1].Key : NAME_None; }
	FText GetSubCategoryText() const { return HasSubCategory() ? CategoryPath[1].Value.GetSubMenuName() : FText::GetEmpty(); }

	UE_API void GetSubCategories(TArray<FName>& SubCategories) const;
	UE_API void GetSubCategoriesText(TArray<FText>& SubCategories) const;
	UE_API void GetSubCategoriesInfo(TArray<FCategoryPath>& SubCategories) const;
	
	FAssetCategoryPath operator / (const FText& SubCategory) const { return FAssetCategoryPath(*this, SubCategory); }
	
private:
	UE_API void AppendSubMenu(const FCategoryPath& InSubMenu);

private:
	TArray<TPair<FName, FCategoryPath>> CategoryPath;
};

#undef UE_API
