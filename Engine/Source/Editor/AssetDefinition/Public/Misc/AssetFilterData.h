// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetCategoryPath.h"

#include "AssetFilterData.generated.h"

USTRUCT()
struct FAssetFilterData
{
	GENERATED_BODY()

public:
	FString Name;
	
	FText DisplayText;

	TArray<FAssetCategoryPath> FilterCategories;
	
	FARFilter Filter;
};
